// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CookMetadata : ModuleRules
{
	public CookMetadata(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core"
			}
		);
	}
}
