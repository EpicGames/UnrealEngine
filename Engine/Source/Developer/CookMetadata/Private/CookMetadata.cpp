// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookMetadata.h"

#include "HAL/FileManager.h"
#include "Hash/xxhash.h"
#include "Internationalization/Internationalization.h"
#include "Memory/MemoryView.h"
#include "Misc/FileHelper.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/LargeMemoryReader.h"

namespace UE::Cook
{

const FString& GetCookMetadataFilename()
{
	static const FString CookMetadataFilename(TEXT("CookMetadata.ucookmeta"));
	return CookMetadataFilename;
}

constexpr uint32 COOK_METADATA_HEADER_MAGIC = 'UCMT';

DEFINE_LOG_CATEGORY_STATIC(LogCookedMetadata, Log, All)


// For backwards compatibility - remove this after 5.4 is released.

/** The name and dependency information for a plugin that was enabled during cooking. */
struct COOKMETADATA_API FCookMetadataPluginEntry_ActualAddShaderPseudoHierarchy
{
	FString Name;
	ECookMetadataPluginType Type = ECookMetadataPluginType::Unassigned;
	TMap<uint8, bool> CustomBoolFields;
	TMap<uint8, FString> CustomStringFields;
	uint32 DependencyIndexStart = 0;
	uint32 DependencyIndexEnd = 0;
	FPluginSizeInfo InclusiveSizes;
	FPluginSizeInfo ExclusiveSizes;

	friend FArchive& operator<<(FArchive& Ar, FCookMetadataPluginEntry_ActualAddShaderPseudoHierarchy& Entry)
	{
		Ar << Entry.Name << Entry.DependencyIndexStart << Entry.DependencyIndexEnd;
		Ar << Entry.InclusiveSizes << Entry.ExclusiveSizes;
		Ar << Entry.CustomBoolFields << Entry.CustomStringFields << Entry.Type;
		return Ar;
	}
};

struct FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy
{
	TArray<FCookMetadataPluginEntry_ActualAddShaderPseudoHierarchy> PluginsEnabledAtCook;
	TArray<uint16> PluginDependencies;
	TArray<uint16> RootPlugins;
	TArray<FString> CustomFieldNames;

	friend FArchive& operator<<(FArchive& Ar, FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy& Hierarchy)
	{
		Ar << Hierarchy.PluginsEnabledAtCook << Hierarchy.PluginDependencies;
		Ar << Hierarchy.RootPlugins << Hierarchy.CustomFieldNames;
		return Ar;
	}
};

static void FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy_To_AdjustCustomFieldLayout(
	FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy& From,
	FCookMetadataPluginHierarchy& To)
{
	To.PluginDependencies = MoveTemp(From.PluginDependencies);
	To.RootPlugins = MoveTemp(From.RootPlugins);
	
	// Now bring over the plugins themselves.
	TMap<uint8, bool> TypeIsBool;
	for (FCookMetadataPluginEntry_ActualAddShaderPseudoHierarchy& OldEntry : From.PluginsEnabledAtCook)
	{
		FCookMetadataPluginEntry& NewEntry = To.PluginsEnabledAtCook.AddDefaulted_GetRef();
		NewEntry.Name = MoveTemp(OldEntry.Name);
		NewEntry.Type = OldEntry.Type;

		for (TPair<uint8, bool>& Bools : OldEntry.CustomBoolFields)
		{
			FCookMetadataPluginEntry::CustomFieldVariantType V;
			V.Set<bool>(Bools.Value);
			NewEntry.CustomFields.Add(Bools.Key, MoveTemp(V));
			TypeIsBool.Add(Bools.Key, true);
		}
		for (TPair<uint8, FString>& Strings : OldEntry.CustomStringFields)
		{
			FCookMetadataPluginEntry::CustomFieldVariantType V;
			V.Emplace<FString>(MoveTemp(Strings.Value));
			NewEntry.CustomFields.Add(Strings.Key, MoveTemp(V));
			TypeIsBool.Add(Strings.Key, false);
		}

		NewEntry.DependencyIndexStart = OldEntry.DependencyIndexStart;
		NewEntry.DependencyIndexEnd = OldEntry.DependencyIndexEnd;
		NewEntry.InclusiveSizes = OldEntry.InclusiveSizes;
		NewEntry.ExclusiveSizes = OldEntry.ExclusiveSizes;
	}

	uint8 EntryIndex = 0;
	for (FString& FieldName : From.CustomFieldNames)
	{
		FCookMetadataPluginHierarchy::FCustomFieldEntry Field = To.CustomFieldEntries.AddDefaulted_GetRef();
		Field.Name = MoveTemp(FieldName);
		Field.Type = ECookMetadataCustomFieldType::Unknown;

		const bool* bIsBool = TypeIsBool.Find(EntryIndex);
		if (bIsBool)
		{
			if (*bIsBool)
			{
				Field.Type = ECookMetadataCustomFieldType::Bool;
			}
			else
			{
				Field.Type = ECookMetadataCustomFieldType::String;
			}
		}

		EntryIndex++;
	}
}

bool FCookMetadataState::Serialize(FArchive& Ar)
{
	uint32 MagicHeader = 0;
	if (Ar.IsLoading())
	{
		Ar << MagicHeader;
		if (MagicHeader != COOK_METADATA_HEADER_MAGIC)
		{
			return false; // not a metadata file.
		}
	}
	else
	{
		MagicHeader = COOK_METADATA_HEADER_MAGIC;
		Ar << MagicHeader;

		Version = ECookMetadataStateVersion::LatestVersion;
	}

	bool bSerializeShaderHierarchy = true;
	Ar << Version;
	if (Ar.IsLoading())
	{
		if (Version < ECookMetadataStateVersion::AddedPluginEntryType)
		{
			UE_LOGF(LogCookedMetadata, Error, "Cook metadata version too old: found %d, we can load %d, latest is %d", EnumToUnderlyingType(Version), EnumToUnderlyingType(ECookMetadataStateVersion::AddedPluginEntryType), EnumToUnderlyingType(ECookMetadataStateVersion::LatestVersion));
			return false; // invalid version - current we don't support backcompat
		}
		if (Version > ECookMetadataStateVersion::LatestVersion)
		{
			UE_LOGF(LogCookedMetadata, Error, "Cook metadata version too new: found %d, latest is %d", EnumToUnderlyingType(Version), EnumToUnderlyingType(ECookMetadataStateVersion::LatestVersion));
			return false;
		}

		bSerializeShaderHierarchy = Version >= ECookMetadataStateVersion::ActualAddShaderPseudoHierarchy;

	}

	if (Ar.IsLoading() && Version <= ECookMetadataStateVersion::ActualAddShaderPseudoHierarchy)
	{
		// Serialize the old hierarchy and convert to the new layout.
		FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy OldHierarchy;
		Ar << OldHierarchy;

		FCookMetadataPluginHierarchy_ActualAddShaderPseudoHierarchy_To_AdjustCustomFieldLayout(
			OldHierarchy, PluginHierarchy);
	}
	else
	{
		// saving, or on a version with the new setup.
		Ar << PluginHierarchy;
	}

	Ar << AssociatedDevelopmentAssetRegistryHash;
	Ar << AssociatedDevelopmentAssetRegistryHashPostWriteback;
	Ar << Platform;
	Ar << BuildVersion;
	Ar << HordeJobId;
	Ar << SizesPresent;

	if (bSerializeShaderHierarchy)
	{
		Ar << ShaderPseudoHierarchy;
	}
	return true;
}

bool FCookMetadataState::ReadFromFile(const FString& FilePath)
{
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*FilePath));
	if (FileReader)
	{
		TArray64<uint8> Data;
		Data.SetNumUninitialized(FileReader->TotalSize());
		FileReader->Serialize(Data.GetData(), Data.Num());
		check(!FileReader->IsError());

		FLargeMemoryReader MemoryReader(Data.GetData(), Data.Num());
		if (Serialize(MemoryReader))
		{
			return true;
		}
		else
		{
			UE_LOGF(LogCookedMetadata, Error, "Failed to serialize cook metadata from file (%ls)", *FilePath);
		}
	}
	else
	{
		UE_LOGF(LogCookedMetadata, Error, "Failed to make file reader from (%ls)", *FilePath);
	}
	return false;
}

bool FCookMetadataState::SaveToFile(const FString& FilePath)
{
	FArrayWriter SerializedCookMetadata;
	Serialize(SerializedCookMetadata);
	if (FFileHelper::SaveArrayToFile(SerializedCookMetadata, *FilePath))
	{
		return true;
	}
	else
	{
		UE_LOGF(LogCookedMetadata, Error, "Failed to write cook metadata file (%ls)", *FilePath);
		return false;
	}
}

/* static */ 
uint64 FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(FMemoryView InSerializedDevelopmentAssetRegistry)
{
	return FXxHash64::HashBufferChunked(InSerializedDevelopmentAssetRegistry.GetData(), InSerializedDevelopmentAssetRegistry.GetSize(), 1ULL << 19).Hash;
}

FText FCookMetadataState::GetSizesPresentAsText() const
{
	// Uncapitalized, presentation-ready
	static FText CookMetadataSizesPresentStrings[] =
	{
		NSLOCTEXT("CookMetadata", "CookMetadataNotPresent", "not present"),
		NSLOCTEXT("CookMetadata", "CookMetadataCompressed", "compressed"),
		NSLOCTEXT("CookMetadata", "CookMetadataUncompressed", "uncompressed")
	};

	static_assert(sizeof(CookMetadataSizesPresentStrings) / sizeof(CookMetadataSizesPresentStrings[0]) == static_cast<size_t>(ECookMetadataSizesPresent::Count));
	return CookMetadataSizesPresentStrings[static_cast<size_t>(SizesPresent)];
}

FText FCookMetadataPluginEntry::GetPluginTypeAsText() const
{
	// Uncapitalized, presentation-ready
	static FText CookMetadataPluginEntryTypeStrings[] =
	{
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeUnassigned", "unknown"),
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeNormal", "normal"),
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeRoot", "root"),
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeEnginePseudo", "engine pseudoplugin"),
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeGamePseudo", "game pseudoplugin"),
		NSLOCTEXT("CookMetadataPluginEntryType", "PluginTypeShaderPseudo", "shader pseudoplugin")
	};

	static_assert(sizeof(CookMetadataPluginEntryTypeStrings) / sizeof(CookMetadataPluginEntryTypeStrings[0]) == static_cast<size_t>(ECookMetadataPluginType::Count));
	return CookMetadataPluginEntryTypeStrings[static_cast<size_t>(Type)];
}
} // end namespace