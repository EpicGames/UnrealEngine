// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Memory/MemoryFwd.h"
#include "Misc/TVariant.h"
#include "UObject/NameTypes.h"

#define UE_API COOKMETADATA_API


namespace UE::Cook
{

enum class ECookMetadataStateVersion : uint8
{
	InvalidVersion = 0,
	PluginHierarchy = 1,
	PostWritebackHash = 2,
	FixSerialization = 3,
	AddedCustomFields = 4,

	// Note: This is a lie, the shader data wasn't actually serializing to file, use ActualAddShaderPseudoHierarchy
	AddedShaderPseudoHierarchy = 5,
	AddedPluginEntryType = 6,
	ActualAddShaderPseudoHierarchy = 7,
	AdjustCustomFieldLayout = 8,

	// Add new versions above this.
	VersionCount,
	LatestVersion = VersionCount - 1
};

/**
*	We classify various bits of the plugin data based on how it will be delivered to the end user in order to
*	more appropriately track the user's experience.
* 
*	Each iostore chunk (note: NOT pak chunk!) gets compressed during staging and unreal pak will update the size
*	for the plugin based on how the chunk gets deployed.
*/
enum class EPluginSizeTypes : uint8
{
	// The iostore chunks will be deployed to servers for downloading on-demand by the game using the Individual Asset Streaming
	// system.
	Streaming,
	// The iostore chunks are written to normal iostore containers that are expected to be distributed with the game. This
	// includes the required global iostore container.
	Installed,
	// The iostore chunks are written to a separate container that isn't required to be distributed with the game. This is where e.g.
	// OptionalMips go for textures. They appear as .uptnl files in the Cooked directory (if cooking to Loose Files),
	// The CopyBuildToStagingDirectory script manually assigns the iostore chunk to a corresponding pak chunk with the name
	// ending in "optional", e.g. pakChunk0optional.pak/ucas/utoc/sig. The only way to catch this in UnrealPak
	// is to parse the filename.
	Optional,
	// These are sidecar files for distributing EditorOnly data for Cooked Editor builds. When cooking to loose files
	// they will contain ".o" inside their filename, e.g. "myasset.o.ubulk". They are never intended to be shipped with
	// a game.
	OptionalSegment,
	COUNT
};
constexpr uint8 EPluginSizeTypesCount = (uint8)EPluginSizeTypes::COUNT;

struct FPluginSizeInfo
{
	uint64 Sizes[EPluginSizeTypesCount] = {};

	void Zero() { FMemory::Memzero(this, sizeof(*this)); }
	void AddSizes(uint64 SizesPerType[EPluginSizeTypesCount])
	{
		for (uint8 Type = 0; Type < EPluginSizeTypesCount; Type++)
		{
			Sizes[Type] += SizesPerType[Type];
		}
	}
	void Add(const FPluginSizeInfo& Other)
	{
		for (uint8 Type = 0; Type < EPluginSizeTypesCount; Type++)
		{
			Sizes[Type] += Other.Sizes[Type];
		}
	}
	uint64 TotalSize() const
	{
		uint64 Total = 0;
		for (uint8 Type = 0; Type < EPluginSizeTypesCount; Type++)
		{
			Total += Sizes[Type];
		}
		return Total;
	}

	friend FArchive& operator<<(FArchive& Ar, FPluginSizeInfo& SizeInfo)
	{
		for (uint8 i = 0; i < EPluginSizeTypesCount; i++)
		{
			Ar << SizeInfo.Sizes[i];
		}
		return Ar;
	}

	uint64& operator[](EPluginSizeTypes InType) { return Sizes[(uint8)InType]; }
	const uint64& operator[](EPluginSizeTypes InType) const { return Sizes[(uint8)InType]; }
};

/*
*	Unrealpak doesn't compress the data for all platforms. In those cases, the data written back
*	during staging isn't compressed and is not representative of the final sizes that occurs after
*	the corresponding SDK tools process them for deployment.
*/
enum class ECookMetadataSizesPresent
{
	// staging has not occured, or writeback wasn't enabled in project packaing settings.
	NotPresent,

	// unrealpak compressed the iostore chunks and the data we have is compressed sizes
	Compressed,

	// the selected platform isn't compressed by unrealpak, or package compression was disabled.
	Uncompressed,

	Count
};

enum class ECookMetadataPluginType
{
	// For sanity tracking. All types _should_ be assigned when they are added.
	Unassigned,
		
	Normal,

	// Root plugins are used to separate and classify game "modes" within a single project.
	Root,

	// For assets under /Engine
	EnginePseudo,

	// For assets under /Game
	GamePseudo,

	// When a shader is referenced by multiple plugins then it has no natural home for assigning
	// its size. Instead we create a set of shader pseudo plugins based on the set of root plugins
	// referencing the shader, including possibly an "Unrooted" plugin.
	ShaderPseudo,

	Count
};

enum class ECookMetadataCustomFieldType : uint8
{
	Unknown,	// This only happens in the upgrade from an older cook metadata
				// when there's nowhere to get the information from. This also means
				// that there are no plugins that use this field so you can safely ignore it.
	Bool,
	String
};

/** The name and dependency information for a plugin that was enabled during cooking. */
struct FCookMetadataPluginEntry
{
	FString Name;

	ECookMetadataPluginType Type = ECookMetadataPluginType::Unassigned;

	// These contain values pulled from the uplugin json file and hold fields that are not
	// part of the engine FPluginDescriptor. They are for per-project values. The keys for the maps
	// are indices in to FCookMetadataPluginHierarchy::CustomFieldEntries, where you'll also find what the type
	// is. See the comment for CustomFieldEntries.
	typedef TVariant<bool, FString> CustomFieldVariantType;
	TMap<uint8, CustomFieldVariantType> CustomFields;

	// The dependencies are stored in the FCookMetadataPluginHierarchy::PluginDependencies array,
	// and this is an index into it. From there you can get a further index into PluginsEnabledAtCook
	// to get the plugin information.
	// Example:
	//	for (uint32 DependencyIndex = Plugin->DependencyIndexStart; DependencyIndex < Plugin->DependencyIndexEnd; DependencyIndex++)
	//	{
	//		const UE::Cook::FCookMetadataPluginEntry& DependentPlugin = PluginHierarchy.PluginsEnabledAtCook[PluginHierarchy.PluginDependencies[DependencyIndex]];
	//	}
	//
	uint32 DependencyIndexStart = 0;
	uint32 DependencyIndexEnd = 0;

	//
	// Theses sizes are set during staging by unrealpak if the option in project packaging is set.
	// To determine if they are set, check FCookMetadataState::GetSizesPresent(). Inclusive contains
	// the size of the plugin and all of its dependencies listed in its uplugin file.
	//
	FPluginSizeInfo InclusiveSizes;
	FPluginSizeInfo ExclusiveSizes;

	uint32 DependencyCount() const { return DependencyIndexEnd - DependencyIndexStart; }

	UE_API FText GetPluginTypeAsText() const;

	// !!! If you edit this, be sure to update the upgrade paths in CookMetadata.cpp
	friend FArchive& operator<<(FArchive& Ar, FCookMetadataPluginEntry& Entry)
	{
		Ar << Entry.Name << Entry.DependencyIndexStart << Entry.DependencyIndexEnd;
		Ar << Entry.InclusiveSizes << Entry.ExclusiveSizes;
		Ar << Entry.CustomFields << Entry.Type;
		return Ar;
	}
};

struct FCookMetadataPluginHierarchy
{
	// The list of plugins that were enabled during the cook that generated the FCookMetadataState
	TArray<FCookMetadataPluginEntry> PluginsEnabledAtCook;

	// The list of plugin dependencies. FCookMetadataPluginEntry::DependencyIndexStart indexes into this
	// array.
	TArray<uint16> PluginDependencies;

	// The list of root plugins for the project as defined by the Editor.ini file.
	TArray<uint16> RootPlugins;

	// The list of custom field names. The values stored on an entry index in to this for the name.
	// These values are copied from the plugins' json descriptor to allow for carrying through project
	// specific data in to the metadata.
	//
	// To enable custom field replication, add the CookMetadataCustomPluginFields section to the relevant Editor.ini file
	// and add to the follow arrays, as desired:
	//
	//  BoolFields, StringFields, PerPlatformBoolFields, PerPlatformStringFields
	// 
	// Bool and String refer to the json type, and will emit as bool or FString here. The PerPlatform arrays
	// will check in the uplugin json for a field named PerPlatform<ArrayValue>. That field is expected to be
	// an array of override objects. Each object is required to have the platform name and override value. E.g.:
	//
	//	[CookMetadataCustomPluginFields]
	//	+PerPlatformBoolFields = ExampleProjectBool
	//	+StringField = ExampleProjectString
	//
	// Will look in the uplugin file for this:
	//
	// 	"ExampleProjectBool": true
	//	"PerPlatformExampleProjectBool": [
	//		{
	//			"Platform": "Windows",
	//			"Value": false
	//		}
	//	]
	//	"ExampleProjectString": "AProjectString"
	//
	// Note that in the per platform case, the base value is not required (i.e. ExampleProjectBool above), however it
	// provides a default value for unlisted platforms. If it does not exist, false is used for bool fields, and an empty string
	// for string fields.
	struct FCustomFieldEntry
	{
		FString Name;
		ECookMetadataCustomFieldType Type;
		friend FArchive& operator<<(FArchive& Ar, FCustomFieldEntry& Entry)
		{
			return Ar << Entry.Name << Entry.Type;
		}
	};
	TArray<FCustomFieldEntry> CustomFieldEntries;

	// !!! If you edit this, be sure to update the upgrade paths in CookMetadata.cpp
	friend FArchive& operator<<(FArchive& Ar, FCookMetadataPluginHierarchy& Hierarchy)
	{
		Ar << Hierarchy.PluginsEnabledAtCook << Hierarchy.PluginDependencies;
		Ar << Hierarchy.RootPlugins << Hierarchy.CustomFieldEntries;
		return Ar;
	}
};

/**
*	After staging when sizes are written back we assign the sizes of shaders
*	to these fake assets and we expose a dependency list for "normal" packages
*	that use them. This can be used to determine an inclusive size for packages
*	that also considers shader sizes.
*/
struct FCookMetadataShaderPseudoAsset
{
	// This is artificially generated based on the chunk the shader belongs to
	// and its hash. It should be consistent across builds. The plugin the shader
	// is assigned to is based on the packages that reference the shader. If the
	// shader is only referenced by packages within a single plugin, the shader asset
	// will be assigned to that plugin. Otherwise, it will be assigned to a shader
	// pseudo plugin.
	FString Name;

	// Shaders are always compressed, independent of what GetSizesPresent() says.
	uint32 CompressedSize = 0;

	friend FArchive& operator<<(FArchive& Ar, FCookMetadataShaderPseudoAsset& PseudoAsset)
	{
		Ar << PseudoAsset.Name << PseudoAsset.CompressedSize;
		return Ar;
	}
};

struct FCookMetadataShaderPseudoHierarchy
{
	TArray<FCookMetadataShaderPseudoAsset> ShaderAssets;

	// Entries in this are indices in to ShaderAssets. Use PackageShaderDependencyMap to
	// get a package's shaders.
	TArray<int32> DependencyList;

	// Keyed off a PackageName in the project, returns a [start, end) pair of indices
	// into DependencyList for the shaders that package depends on.
	TMap<FName, TPair<int32, int32>> PackageShaderDependencyMap;

	friend FArchive& operator<<(FArchive& Ar, FCookMetadataShaderPseudoHierarchy& PseudoHierarchy)
	{
		Ar << PseudoHierarchy.ShaderAssets << PseudoHierarchy.DependencyList << PseudoHierarchy.PackageShaderDependencyMap;
		return Ar;
	}
};

/**
*	Structure serialized to disk to contain non-asset related metadata about a cook. This should always
*	exist alongside a Development Asset Registry, and to ensure that the pair is not out of sync, users
*	should validate the development asset registry they are using with GetAssociatedDevelopmentAssetRegistryHash().
*/
class FCookMetadataState
{
public:
	FCookMetadataState() = default;
	FCookMetadataState(const FCookMetadataState&) = default;
	FCookMetadataState(FCookMetadataState&& Rhs) = default;
	~FCookMetadataState() = default;

	FCookMetadataState& operator=(const FCookMetadataState&) = default;
	FCookMetadataState& operator=(FCookMetadataState&& O) = default;

	bool IsValid() const { return Version == ECookMetadataStateVersion::LatestVersion; }
	void Reset() { *this = FCookMetadataState(); }
	
	UE_API bool Serialize(FArchive& Ar);
	UE_API bool ReadFromFile(const FString& FilePath);
	UE_API bool SaveToFile(const FString& FilePath);


	// Plugin hierarchy information
	void SetPluginHierarchyInfo(FCookMetadataPluginHierarchy&& InPluginHierarchy) { PluginHierarchy = MoveTemp(InPluginHierarchy); }
	const FCookMetadataPluginHierarchy& GetPluginHierarchy() const { return PluginHierarchy; }
	
	// So that unrealpak can update the sizes.
	FCookMetadataPluginHierarchy& GetMutablePluginHierarchy() { return PluginHierarchy; }

	/**
	*	Associated DevAR Hash.
	* 
	*	This is computed by reading the development asset registry file into a memory buffer and calling
	*	ComputeHashOfDevelopmentAssetRegistry on the data.
	* 
	*	Use this to ensure that the files you are working with were produced by the same cook and didn't
	*	get out of sync somehow.
	* 
	*	*IMPORTANT* If asset registry writeback is enabled during staging, then the hash of the development
	*	asset registry changes, and you'll need to check against GetAssociatedDevelopmentAssetRegistryHashPostWriteback.
	*	If you don't know which one you have, check both - they are both valid.
	* 
	*	e.g.
	*	uint64 CheckHash = GetAssociatedDevelopmentAssetRegistryHash();
	*	bValidDevAr = ComputeHashOfDevelopmentAssetRegistry(MakeMemoryView(SerializedAssetRegistry)) == CheckHash;
	*/
	void SetAssociatedDevelopmentAssetRegistryHash(uint64 InHash) { AssociatedDevelopmentAssetRegistryHash = InHash; }
	void SetAssociatedDevelopmentAssetRegistryHashPostWriteback(uint64 InHash) { AssociatedDevelopmentAssetRegistryHashPostWriteback = InHash; }

	uint64 GetAssociatedDevelopmentAssetRegistryHash() const { return AssociatedDevelopmentAssetRegistryHash; }
	uint64 GetAssociatedDevelopmentAssetRegistryHashPostWriteback() const { return AssociatedDevelopmentAssetRegistryHashPostWriteback; }

	void SetPlatformAndBuildVersion(const FString& InPlatform, const TCHAR* InBuildVersion) { Platform = InPlatform; BuildVersion = FString(InBuildVersion); }
	const FString& GetPlatform() const { return Platform; }
	const FString GetBuildVersion() const { return BuildVersion; }

	void SetHordeJobId(FString&& InHordeJobId) { HordeJobId = MoveTemp(InHordeJobId); }
	const FString& GetHordeJobId() const { return HordeJobId; }

	static UE_API uint64 ComputeHashOfDevelopmentAssetRegistry(FMemoryView InSerializedDevelopmentAssetRegistry);

	// Returns what size information is present in FCookMetadataPluginEntry. This varies based on the platform
	// and settings. 
	UE_API FText GetSizesPresentAsText() const;
	ECookMetadataSizesPresent GetSizesPresent() const { return SizesPresent; }
	void SetSizesPresent(ECookMetadataSizesPresent InSizesPresent) { SizesPresent = InSizesPresent; }

	void SetShaderPseudoHieararchy(FCookMetadataShaderPseudoHierarchy&& InHierarchy) { ShaderPseudoHierarchy = MoveTemp(InHierarchy); }
	const FCookMetadataShaderPseudoHierarchy& GetShaderPseudoHierarchy() const { return ShaderPseudoHierarchy; }
private:

	ECookMetadataStateVersion Version = ECookMetadataStateVersion::InvalidVersion;
	FCookMetadataPluginHierarchy PluginHierarchy;

	FCookMetadataShaderPseudoHierarchy ShaderPseudoHierarchy;

	uint64 AssociatedDevelopmentAssetRegistryHash = 0;

	// Asset registry size writeback changes the AR, so we have a separate hash for that DevAR that this
	// also matches.
	uint64 AssociatedDevelopmentAssetRegistryHashPostWriteback = 0;

	FString Platform;

	// BUILD_VERSION definition from definitions.h for the cook.
	FString BuildVersion;

	// If cooked on Horde, this is the job id that cooked it.
	FString HordeJobId;

	// Updated by unrealpak when plugin size information is added.
	ECookMetadataSizesPresent SizesPresent = ECookMetadataSizesPresent::NotPresent;
};

COOKMETADATA_API const FString& GetCookMetadataFilename();

}; // namespace

#undef UE_API
